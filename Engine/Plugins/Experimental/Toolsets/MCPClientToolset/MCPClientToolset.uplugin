{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "MCP Client Toolset",
	"Description": "An adapter that allows toolset registry customers (like the EDA) to connect to local/private MCP servers.",
	"Category": "Experimental",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EditorOnly": true,
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Plugins": [
		{
			"Name": "ToolsetRegistry",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "MCPClientToolset",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"TargetAllowList": [ "Editor" ]
		}
	]
}
