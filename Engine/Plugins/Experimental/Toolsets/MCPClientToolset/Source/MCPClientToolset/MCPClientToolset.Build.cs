// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MCPClientToolset : ModuleRules
{
	public MCPClientToolset(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		IWYUSupport = IWYUSupport.Full;
		bUseUnity = true;

		PublicDefinitions.Add("WITH_AIASSISTANT_EPIC_INTERNAL=1");

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"ToolsetRegistry",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"DeveloperSettings",
				"EditorSubsystem",
				"Engine",
				"HTTP",
				"HTTPServer",
				"Json",
				"UnrealEd",
			});
	}
}
