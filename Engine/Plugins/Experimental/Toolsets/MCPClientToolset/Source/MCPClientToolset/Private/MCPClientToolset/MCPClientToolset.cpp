// Copyright Epic Games, Inc. All Rights Reserved.

#include "MCPClientToolset/MCPClientToolset.h"

#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "HAL/PlatformProcess.h"
#include "HttpModule.h"
#include "HttpPath.h"
#include "HttpRequestHandler.h"
#include "HttpResultCallback.h"
#include "HttpRouteHandle.h"
#include "IHttpRouter.h"
#include "HttpServerModule.h"
#include "HttpServerRequest.h"
#include "HttpServerResponse.h"
#include "Interfaces/IHttpResponse.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/Base64.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Guid.h"
#include "PlatformHttp.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "MCPClientToolset/Module.h"

// ---------------------------------------------------------------------------
// Minimal self-contained SHA-256
// FPlatformMisc::GetSHA256Signature is not implemented on all platforms.
// ---------------------------------------------------------------------------
namespace
{
	static uint32 SHA256_ROTR(uint32 x, uint32 n) { return (x >> n) | (x << (32u - n)); }

	static void ComputeSHA256(const uint8* Msg, uint32 MsgLen, uint8 OutHash[32])
	{
		static const uint32 K[64] = {
			0x428a2f98,0x71374491,0xb5c0fbcf,0xe9b5dba5,0x3956c25b,0x59f111f1,0x923f82a4,0xab1c5ed5,
			0xd807aa98,0x12835b01,0x243185be,0x550c7dc3,0x72be5d74,0x80deb1fe,0x9bdc06a7,0xc19bf174,
			0xe49b69c1,0xefbe4786,0x0fc19dc6,0x240ca1cc,0x2de92c6f,0x4a7484aa,0x5cb0a9dc,0x76f988da,
			0x983e5152,0xa831c66d,0xb00327c8,0xbf597fc7,0xc6e00bf3,0xd5a79147,0x06ca6351,0x14292967,
			0x27b70a85,0x2e1b2138,0x4d2c6dfc,0x53380d13,0x650a7354,0x766a0abb,0x81c2c92e,0x92722c85,
			0xa2bfe8a1,0xa81a664b,0xc24b8b70,0xc76c51a3,0xd192e819,0xd6990624,0xf40e3585,0x106aa070,
			0x19a4c116,0x1e376c08,0x2748774c,0x34b0bcb5,0x391c0cb3,0x4ed8aa4a,0x5b9cca4f,0x682e6ff3,
			0x748f82ee,0x78a5636f,0x84c87814,0x8cc70208,0x90befffa,0xa4506ceb,0xbef9a3f7,0xc67178f2
		};

		uint32 H[8] = {
			0x6a09e667,0xbb67ae85,0x3c6ef372,0xa54ff53a,
			0x510e527f,0x9b05688c,0x1f83d9ab,0x5be0cd19
		};

		const uint32 PaddedLen = ((MsgLen + 9 + 63) / 64) * 64;
		TArray<uint8> Padded;
		Padded.SetNumZeroed(PaddedLen);
		FMemory::Memcpy(Padded.GetData(), Msg, MsgLen);
		Padded[MsgLen] = 0x80;
		const uint64 BitLen = (uint64)MsgLen * 8;
		for (int32 i = 0; i < 8; ++i)
		{
			Padded[PaddedLen - 8 + i] = (uint8)(BitLen >> (56 - i * 8));
		}

		for (uint32 Off = 0; Off < PaddedLen; Off += 64)
		{
			uint32 W[64];
			for (int32 i = 0; i < 16; ++i)
			{
				W[i] = ((uint32)Padded[Off+i*4]<<24)|((uint32)Padded[Off+i*4+1]<<16)
				      |((uint32)Padded[Off+i*4+2]<<8)|(uint32)Padded[Off+i*4+3];
			}
			for (int32 i = 16; i < 64; ++i)
			{
				const uint32 s0 = SHA256_ROTR(W[i-15],7)^SHA256_ROTR(W[i-15],18)^(W[i-15]>>3);
				const uint32 s1 = SHA256_ROTR(W[i-2],17)^SHA256_ROTR(W[i-2],19)^(W[i-2]>>10);
				W[i] = W[i-16] + s0 + W[i-7] + s1;
			}
			uint32 a=H[0],b=H[1],c=H[2],d=H[3],e=H[4],f=H[5],g=H[6],h=H[7];
			for (int32 i = 0; i < 64; ++i)
			{
				const uint32 S1  = SHA256_ROTR(e,6)^SHA256_ROTR(e,11)^SHA256_ROTR(e,25);
				const uint32 Ch  = (e&f)^(~e&g);
				const uint32 T1  = h + S1 + Ch + K[i] + W[i];
				const uint32 S0  = SHA256_ROTR(a,2)^SHA256_ROTR(a,13)^SHA256_ROTR(a,22);
				const uint32 Maj = (a&b)^(a&c)^(b&c);
				const uint32 T2  = S0 + Maj;
				h=g; g=f; f=e; e=d+T1; d=c; c=b; b=a; a=T1+T2;
			}
			H[0]+=a; H[1]+=b; H[2]+=c; H[3]+=d;
			H[4]+=e; H[5]+=f; H[6]+=g; H[7]+=h;
		}

		for (int32 i = 0; i < 8; ++i)
		{
			OutHash[i*4]   = (uint8)(H[i]>>24);
			OutHash[i*4+1] = (uint8)(H[i]>>16);
			OutHash[i*4+2] = (uint8)(H[i]>>8);
			OutHash[i*4+3] = (uint8)(H[i]);
		}
	}
} // anonymous namespace

namespace UE::ToolsetRegistry
{

// ---------------------------------------------------------------------------
// Construction / destruction
// ---------------------------------------------------------------------------

FMCPClientToolset::FMCPClientToolset(FPrivateToken, FConfig InConfig)
	: Config(MoveTemp(InConfig))
{
	// Trim whitespace/newlines that can creep in via copy-paste in editor settings.
	// libcurl rejects URLs with trailing whitespace with CURLE_URL_MALFORMAT (error 3).
	Config.ServerUrl.TrimStartAndEndInline();
	Config.ApiKey.TrimStartAndEndInline();
	Config.Description.TrimStartAndEndInline();
	Config.OAuthClientId.TrimStartAndEndInline();
	Config.OAuthScope.TrimStartAndEndInline();
}

FMCPClientToolset::~FMCPClientToolset()
{
	// Fulfill the create promise before it destructs — an unfulfilled TPromise fires a check().
	if (CreatePromise.IsValid())
	{
		CreatePromise->SetValue(
			MakeError(FString::Printf(TEXT("MCP toolset '%s' was destroyed before handshake completed"),
				*Config.Name)));
		CreatePromise.Reset();
	}

	// OAuth cleanup: stop the local HTTP listener if it was started and not yet cleaned up.
	FTSTicker::GetCoreTicker().RemoveTicker(OAuthTimeoutHandle);
	if (OAuthRouter.IsValid())
	{
		if (OAuthRouteHandle.IsValid())
		{
			OAuthRouter->UnbindRoute(OAuthRouteHandle);
		}
		OAuthRouter.Reset();
	}

	FTSTicker::GetCoreTicker().RemoveTicker(SSETickerHandle);
	SSEStreamState.Reset(); // stop HTTP-thread stream callback from enqueuing after destruction

	if (SSERequest.IsValid())
	{
		SSERequest->CancelRequest();
		SSERequest.Reset();
	}
	RejectAllPendingRequests();
}

// ---------------------------------------------------------------------------
// Public factory
// ---------------------------------------------------------------------------

TFuture<TValueOrError<TSharedPtr<FMCPClientToolset>, FString>> FMCPClientToolset::Create(FConfig Config)
{
	TSharedPtr<FMCPClientToolset> Toolset = MakeShared<FMCPClientToolset>(FPrivateToken{}, MoveTemp(Config));

	auto Promise = MakeShared<TPromise<TValueOrError<TSharedPtr<FMCPClientToolset>, FString>>>();
	TFuture<TValueOrError<TSharedPtr<FMCPClientToolset>, FString>> Future = Promise->GetFuture();

	Toolset->CreatePromise = Promise;
	Toolset->Self = Toolset; // keep alive until CreatePromise resolves

	if (Toolset->Config.bOAuth)
	{
		Toolset->StartOAuthFlow();
	}
	else if (Toolset->Config.bStreamableHTTP)
	{
		Toolset->StartStreamableHTTPHandshake();
	}
	else
	{
		Toolset->StartSSEConnection();
	}

	return Future;
}

// ---------------------------------------------------------------------------
// FToolset interface
// ---------------------------------------------------------------------------

FString FMCPClientToolset::GetToolsetName() const
{
	return Config.Name;
}

FString FMCPClientToolset::GetToolsetVersion() const
{
	return ServerVersion.IsEmpty() ? TEXT("1.0") : ServerVersion;
}

FString FMCPClientToolset::GetToolsetDescription() const
{
	return Config.Description;
}

FString FMCPClientToolset::GetJsonSchemaInternal() const
{
	return CachedSchemaJson;
}

TFuture<TValueOrError<FString, FString>> FMCPClientToolset::ExecuteToolInternal(
	const FString& ToolName, const FString& JsonInput)
{
	// Parse optional JSON input
	TSharedPtr<FJsonObject> InputJson;
	if (!JsonInput.IsEmpty())
	{
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(JsonInput);
		FJsonSerializer::Deserialize(Reader, InputJson);
	}
	if (!InputJson.IsValid())
	{
		InputJson = MakeShared<FJsonObject>();
	}

	// Build tools/call params
	TSharedPtr<FJsonObject> Params = MakeShared<FJsonObject>();
	Params->SetStringField(TEXT("name"), ToolName);
	Params->SetObjectField(TEXT("arguments"), InputJson);

	return SendRequest(TEXT("tools/call"), Params)
		.Next([WeakThis = AsWeak()](TSharedPtr<FJsonObject> Response) -> TValueOrError<FString, FString>
		{
			TSharedPtr<FMCPClientToolset> StrongThis = WeakThis.Pin();
			if (!StrongThis.IsValid())
			{
				return MakeError(FString(TEXT("MCP toolset was destroyed before tools/call completed")));
			}

			if (!Response.IsValid())
			{
				return MakeError(FString(TEXT("tools/call failed: no response received")));
			}

			// JSON-RPC error object
			const TSharedPtr<FJsonObject>* ErrorObj = nullptr;
			if (Response->TryGetObjectField(TEXT("error"), ErrorObj) && ErrorObj)
			{
				FString ErrorMsg;
				(*ErrorObj)->TryGetStringField(TEXT("message"), ErrorMsg);
				return MakeError(ErrorMsg.IsEmpty() ? FString(TEXT("Unknown MCP error")) : ErrorMsg);
			}

			// Extract result
			const TSharedPtr<FJsonObject>* ResultObj = nullptr;
			if (Response->TryGetObjectField(TEXT("result"), ResultObj) && ResultObj)
			{
				return MakeValue(StrongThis->MCPResultToString(*ResultObj));
			}

			return MakeError(FString(TEXT("tools/call: response contained no result")));
		});
}

// ---------------------------------------------------------------------------
// Auth header
// ---------------------------------------------------------------------------

FString FMCPClientToolset::GetAuthHeader() const
{
	if (!AccessToken.IsEmpty())
	{
		return TEXT("Bearer ") + AccessToken;
	}
	if (!Config.ApiKey.IsEmpty())
	{
		return TEXT("Bearer ") + Config.ApiKey;
	}
	return FString();
}

// ---------------------------------------------------------------------------
// SSE connection
// ---------------------------------------------------------------------------

void FMCPClientToolset::StartSSEConnection()
{
	const FString SSEUrl = Config.ServerUrl + TEXT("/sse");
	UE_LOGF(LogMCPClientToolset, Log, "MCPClientToolset '%ls': connecting to %ls", *Config.Name, *SSEUrl);

	SSERequest = FHttpModule::Get().CreateRequest();
	SSERequest->SetURL(SSEUrl);
	SSERequest->SetVerb(TEXT("GET"));
	SSERequest->SetHeader(TEXT("Accept"), TEXT("text/event-stream"));
	SSERequest->SetHeader(TEXT("Cache-Control"), TEXT("no-cache"));
	SSERequest->SetHeader(TEXT("Connection"), TEXT("keep-alive"));
	SSERequest->SetHeader(TEXT("Accept-Encoding"), TEXT("identity")); // prevent compression on the stream

	const FString AuthHeader = GetAuthHeader();
	if (!AuthHeader.IsEmpty())
	{
		SSERequest->SetHeader(TEXT("Authorization"), AuthHeader);
	}

	// SetResponseBodyReceiveStreamDelegateV2 is the only reliable way to receive streaming
	// data from a long-lived SSE connection. Once it is set, GetContent() always returns
	// empty (documented UE behaviour), so any polling approach on GetContent() cannot work.
	//
	// The stream delegate fires on the HTTP thread (not the game thread). We enqueue raw
	// byte chunks into a thread-safe SPSC queue and drain it on the game thread via
	// FTSTicker, avoiding any cross-thread race with OnProcessRequestComplete.
	SSEStreamState = MakeShared<FSSEStreamState, ESPMode::ThreadSafe>();

	// Capture the shared state by value — safe from the HTTP thread because
	// TSharedPtr<T, ESPMode::ThreadSafe> uses atomic reference counting.
	TSharedPtr<FSSEStreamState, ESPMode::ThreadSafe> CapturedState = SSEStreamState;
	const bool bStreamSet = SSERequest->SetResponseBodyReceiveStreamDelegateV2(
		FHttpRequestStreamDelegateV2::CreateLambda(
			[CapturedState](void* Ptr, int64& InOutLength)
			{
				if (CapturedState.IsValid() && InOutLength > 0)
				{
					TArray<uint8> Chunk;
					Chunk.Append(static_cast<const uint8*>(Ptr), static_cast<int32>(InOutLength));
					CapturedState->Queue.Enqueue(MoveTemp(Chunk));
				}
			}));
	if (!bStreamSet)
	{
		UE_LOGF(LogMCPClientToolset, Warning,
			"MCPClientToolset '%ls': SetResponseBodyReceiveStreamDelegateV2 failed — SSE streaming unavailable",
			*Config.Name);
	}

	// Drain the SPSC queue on the game thread each frame so SSE events are dispatched there.
	SSETickerHandle = FTSTicker::GetCoreTicker().AddTicker(
		FTickerDelegate::CreateSP(AsShared(), &FMCPClientToolset::OnSSETick));

	SSERequest->OnProcessRequestComplete().BindLambda(
		[WeakThis = AsWeak()](FHttpRequestPtr Request, FHttpResponsePtr Response, bool bConnectedSuccessfully)
		{
			if (TSharedPtr<FMCPClientToolset> StrongThis = WeakThis.Pin())
			{
				StrongThis->OnSSEComplete(Request, Response, bConnectedSuccessfully);
			}
		});

	SSERequest->ProcessRequest();
}

bool FMCPClientToolset::OnSSETick(float /*DeltaTime*/)
{
	if (!SSEStreamState.IsValid())
	{
		return false; // state gone (connection closed), stop ticking
	}

	// Drain all chunks enqueued by the HTTP-thread stream callback.
	TArray<uint8> Chunk;
	while (SSEStreamState->Queue.Dequeue(Chunk))
	{
		if (Chunk.Num() > 0)
		{
			Chunk.Add(0); // null-terminate for UTF8_TO_TCHAR
			SSEBuffer += UTF8_TO_TCHAR(reinterpret_cast<const ANSICHAR*>(Chunk.GetData()));
			FlushSSEBuffer();
		}
	}

	return true; // keep ticking until OnSSEComplete removes this ticker
}

void FMCPClientToolset::OnSSEComplete(
	FHttpRequestPtr /*Request*/, FHttpResponsePtr Response, bool bConnectedSuccessfully)
{
	const int32 ResponseCode = (Response.IsValid()) ? Response->GetResponseCode() : 0;

	// Stop the per-frame drain ticker — connection is done.
	FTSTicker::GetCoreTicker().RemoveTicker(SSETickerHandle);

	UE_LOGF(LogMCPClientToolset, Log,
		"MCPClientToolset '%ls': SSE request ended — connected=%d HTTP=%d",
		*Config.Name, bConnectedSuccessfully ? 1 : 0, ResponseCode);

	// Drain any chunks the HTTP-thread stream callback enqueued before this completion
	// callback ran. OnProcessRequestComplete fires on the game thread, so no race here.
	if (SSEStreamState.IsValid())
	{
		TArray<uint8> Chunk;
		while (SSEStreamState->Queue.Dequeue(Chunk))
		{
			if (Chunk.Num() > 0)
			{
				Chunk.Add(0);
				SSEBuffer += UTF8_TO_TCHAR(reinterpret_cast<const ANSICHAR*>(Chunk.GetData()));
				FlushSSEBuffer();
			}
		}
		SSEStreamState.Reset();
	}

	// Reject any pending JSON-RPC requests — their responses can no longer arrive via SSE.
	// Must happen before Self.Reset(), which may destroy *this.
	RejectAllPendingRequests();

	// Reject the create handshake if it still hasn't resolved.
	// Self.Reset() is last: it may drop the ref count to zero and destroy *this,
	// so no member access is allowed after it.
	if (CreatePromise.IsValid())
	{
		CreatePromise->SetValue(
			MakeError(FString::Printf(
				TEXT("SSE connection to '%s' closed before handshake completed (HTTP %d)"),
				*Config.Name, ResponseCode)));
		CreatePromise.Reset();
	}
	Self.Reset();
}

// ---------------------------------------------------------------------------
// SSE parsing
// ---------------------------------------------------------------------------

void FMCPClientToolset::FlushSSEBuffer()
{
	// SSE events are delimited by a blank line — either \n\n (LF) or \r\n\r\n (CRLF).
	// Each iteration extracts one complete event from the front of SSEBuffer.
	while (true)
	{
		// Pick whichever delimiter appears first so mixed line-ending streams work.
		const int32 LFIdx   = SSEBuffer.Find(TEXT("\n\n"),     ESearchCase::CaseSensitive);
		const int32 CRLFIdx = SSEBuffer.Find(TEXT("\r\n\r\n"), ESearchCase::CaseSensitive);

		int32 DelimIdx    = INDEX_NONE;
		int32 DelimLength = 0;

		if (LFIdx != INDEX_NONE && (CRLFIdx == INDEX_NONE || LFIdx <= CRLFIdx))
		{
			DelimIdx    = LFIdx;
			DelimLength = 2; // \n\n
		}
		else if (CRLFIdx != INDEX_NONE)
		{
			DelimIdx    = CRLFIdx;
			DelimLength = 4; // \r\n\r\n
		}

		if (DelimIdx == INDEX_NONE)
		{
			break; // no complete event yet — keep buffering
		}

		const FString EventBlock = SSEBuffer.Left(DelimIdx);
		SSEBuffer = SSEBuffer.Mid(DelimIdx + DelimLength);

		FString EventType = TEXT("message"); // SSE default
		FString EventData;

		TArray<FString> Lines;
		EventBlock.ParseIntoArrayLines(Lines, /*bCullEmpty=*/false);

		for (const FString& Line : Lines)
		{
			if (Line.StartsWith(TEXT("event:"), ESearchCase::CaseSensitive))
			{
				EventType = Line.Mid(6).TrimStartAndEnd(); // TrimStartAndEnd handles trailing \r
			}
			else if (Line.StartsWith(TEXT("data:"), ESearchCase::CaseSensitive))
			{
				EventData = Line.Mid(5).TrimStartAndEnd();
			}
			// Ignore "id:", "retry:", and comment lines (":...")
		}

		if (!EventData.IsEmpty())
		{
			DispatchSSEEvent(EventType, EventData);
		}
	}
}

void FMCPClientToolset::DispatchSSEEvent(const FString& EventType, const FString& Data)
{
	if (EventType == TEXT("endpoint"))
	{
		// Data is a relative path like "/message?sessionId=abc".
		// Construct full MessageEndpointUrl from Config.ServerUrl + path.
		FString Base = Config.ServerUrl;
		if (Base.EndsWith(TEXT("/")))
		{
			Base = Base.LeftChop(1);
		}

		if (Data.StartsWith(TEXT("/")))
		{
			// Strip any existing path from base URL (keep scheme + host + port only).
			// e.g. "http://localhost:3000/some/path" → "http://localhost:3000" + Data
			const int32 SchemeDelim = Base.Find(TEXT("://"), ESearchCase::CaseSensitive);
			if (SchemeDelim != INDEX_NONE)
			{
				const int32 AfterScheme = SchemeDelim + 3; // skip "://"
				const int32 PathStart   = Base.Find(TEXT("/"), ESearchCase::CaseSensitive,
				                                    ESearchDir::FromStart, AfterScheme);
				MessageEndpointUrl = (PathStart != INDEX_NONE)
					? Base.Left(PathStart) + Data
					: Base + Data;
			}
			else
			{
				MessageEndpointUrl = Base + Data;
			}
		}
		else
		{
			MessageEndpointUrl = Data;
		}

		UE_LOGF(LogMCPClientToolset, Log,
			"MCPClientToolset '%ls': message endpoint = %ls", *Config.Name, *MessageEndpointUrl);

		// ---- Step 2: send initialize ----
		TSharedPtr<FJsonObject> InitParams = MakeShared<FJsonObject>();
		InitParams->SetStringField(TEXT("protocolVersion"), TEXT("2024-11-05"));
		InitParams->SetObjectField(TEXT("capabilities"), MakeShared<FJsonObject>());

		TSharedPtr<FJsonObject> ClientInfo = MakeShared<FJsonObject>();
		ClientInfo->SetStringField(TEXT("name"),    TEXT("UE-ToolsetRegistry"));
		ClientInfo->SetStringField(TEXT("version"), TEXT("1.0"));
		InitParams->SetObjectField(TEXT("clientInfo"), ClientInfo);

		SendRequest(TEXT("initialize"), InitParams)
			.Next([WeakThis = AsWeak()](TSharedPtr<FJsonObject> InitResponse)
			{
				TSharedPtr<FMCPClientToolset> StrongThis = WeakThis.Pin();
				if (!StrongThis.IsValid())
				{
					return;
				}

				if (!InitResponse.IsValid())
				{
					if (StrongThis->CreatePromise.IsValid())
					{
						StrongThis->CreatePromise->SetValue(
							MakeError(FString::Printf(TEXT("MCP initialize failed for '%s'"), *StrongThis->Config.Name)));
						StrongThis->CreatePromise.Reset();
					}
					StrongThis->Self.Reset();
					return;
				}

				// Cache server version from result.serverInfo
				{
					const TSharedPtr<FJsonObject>* ResultObj = nullptr;
					if (InitResponse->TryGetObjectField(TEXT("result"), ResultObj) && ResultObj)
					{
						const TSharedPtr<FJsonObject>* ServerInfoObj = nullptr;
						if ((*ResultObj)->TryGetObjectField(TEXT("serverInfo"), ServerInfoObj) && ServerInfoObj)
						{
							(*ServerInfoObj)->TryGetStringField(TEXT("version"), StrongThis->ServerVersion);
						}
					}
				}

				// ---- Step 3: send notifications/initialized (fire-and-forget) ----
				{
					TSharedPtr<IHttpRequest, ESPMode::ThreadSafe> NotifRequest = FHttpModule::Get().CreateRequest();
					NotifRequest->SetURL(StrongThis->MessageEndpointUrl);
					NotifRequest->SetVerb(TEXT("POST"));
					NotifRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json"));

					const FString AuthHdr = StrongThis->GetAuthHeader();
					if (!AuthHdr.IsEmpty())
					{
						NotifRequest->SetHeader(TEXT("Authorization"), AuthHdr);
					}

					TSharedPtr<FJsonObject> Notif = MakeShared<FJsonObject>();
					Notif->SetStringField(TEXT("jsonrpc"), TEXT("2.0"));
					Notif->SetStringField(TEXT("method"),  TEXT("notifications/initialized"));

					FString NotifBody;
					TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&NotifBody);
					FJsonSerializer::Serialize(Notif.ToSharedRef(), Writer);

					NotifRequest->SetContentAsString(NotifBody);
					NotifRequest->ProcessRequest();
				}

				// ---- Step 4: send tools/list ----
				StrongThis->SendRequest(TEXT("tools/list"), nullptr)
					.Next([WeakThis](TSharedPtr<FJsonObject> ToolsResponse)
					{
						TSharedPtr<FMCPClientToolset> StrongThis2 = WeakThis.Pin();
						if (!StrongThis2.IsValid())
						{
							return;
						}

						if (!ToolsResponse.IsValid())
						{
							if (StrongThis2->CreatePromise.IsValid())
							{
								StrongThis2->CreatePromise->SetValue(
									MakeError(FString::Printf(TEXT("MCP tools/list failed for '%s'"), *StrongThis2->Config.Name)));
								StrongThis2->CreatePromise.Reset();
							}
							StrongThis2->Self.Reset();
							return;
						}

						// Extract tools array from result
						TArray<TSharedPtr<FJsonValue>> Tools;
						{
							const TSharedPtr<FJsonObject>* ResultObj = nullptr;
							if (ToolsResponse->TryGetObjectField(TEXT("result"), ResultObj) && ResultObj)
							{
								const TArray<TSharedPtr<FJsonValue>>* ToolsArray = nullptr;
								if ((*ResultObj)->TryGetArrayField(TEXT("tools"), ToolsArray) && ToolsArray)
								{
									Tools = *ToolsArray;
								}
							}
						}

						// ---- Step 5: build schema and resolve Create() ----
						StrongThis2->CachedSchemaJson = StrongThis2->BuildSchemaFromMCPTools(Tools);

						UE_LOGF(LogMCPClientToolset, Log,
							"MCPClientToolset '%ls': registered %d tool(s)",
							*StrongThis2->Config.Name, Tools.Num());

						if (StrongThis2->CreatePromise.IsValid())
						{
							StrongThis2->CreatePromise->SetValue(MakeValue(StrongThis2));
							StrongThis2->CreatePromise.Reset();
						}
						StrongThis2->Self.Reset();
					});
			});
	}
	else if (EventType == TEXT("message"))
	{
		// Parse JSON-RPC response and resolve the matching pending request
		TSharedPtr<FJsonObject> JsonResponse;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Data);
		if (!FJsonSerializer::Deserialize(Reader, JsonResponse) || !JsonResponse.IsValid())
		{
			UE_LOGF(LogMCPClientToolset, Warning,
				"MCPClientToolset '%ls': failed to parse SSE message JSON", *Config.Name);
			return;
		}

		// The id can be a number or a string in JSON-RPC 2.0
		int32 RequestId = -1;
		{
			double IdDouble = 0.0;
			if (JsonResponse->TryGetNumberField(TEXT("id"), IdDouble))
			{
				RequestId = static_cast<int32>(IdDouble);
			}
			else
			{
				FString IdStr;
				if (JsonResponse->TryGetStringField(TEXT("id"), IdStr))
				{
					RequestId = FCString::Atoi(*IdStr);
				}
			}
		}

		if (TSharedPtr<TPromise<TSharedPtr<FJsonObject>>>* PromisePtr = PendingRequests.Find(RequestId))
		{
			(*PromisePtr)->SetValue(JsonResponse);
			PendingRequests.Remove(RequestId);
		}
	}
	// Other event types (e.g., "ping", "error") are silently ignored
}

// ---------------------------------------------------------------------------
// JSON-RPC helpers
// ---------------------------------------------------------------------------

TFuture<TSharedPtr<FJsonObject>> FMCPClientToolset::SendRequest(
	const FString& Method, TSharedPtr<FJsonObject> Params)
{
	// Streamable HTTP transport: delegate to the per-request POST implementation.
	if (Config.bStreamableHTTP)
	{
		return SendStreamableHTTPRequest(Method, Params);
	}

	// Legacy SSE transport: POST to MessageEndpointUrl; response arrives via SSE.
	auto Promise = MakeShared<TPromise<TSharedPtr<FJsonObject>>>();
	TFuture<TSharedPtr<FJsonObject>> Future = Promise->GetFuture();

	if (MessageEndpointUrl.IsEmpty())
	{
		UE_LOGF(LogMCPClientToolset, Warning,
			"MCPClientToolset '%ls': SendRequest called before endpoint is known (method=%ls)",
			*Config.Name, *Method);
		Promise->SetValue(nullptr);
		return Future;
	}

	const int32 RequestId = NextRequestId++;

	// Build JSON-RPC 2.0 request
	TSharedPtr<FJsonObject> JsonRpc = MakeShared<FJsonObject>();
	JsonRpc->SetStringField(TEXT("jsonrpc"), TEXT("2.0"));
	JsonRpc->SetNumberField(TEXT("id"), RequestId);
	JsonRpc->SetStringField(TEXT("method"), Method);
	if (Params.IsValid())
	{
		JsonRpc->SetObjectField(TEXT("params"), Params);
	}

	FString RequestBody;
	{
		TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&RequestBody);
		FJsonSerializer::Serialize(JsonRpc.ToSharedRef(), Writer);
	}

	// Store promise before firing the request so the SSE response can find it
	PendingRequests.Add(RequestId, Promise);

	// Build and fire the HTTP POST
	TSharedPtr<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();
	HttpRequest->SetURL(MessageEndpointUrl);
	HttpRequest->SetVerb(TEXT("POST"));
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json"));

	const FString AuthHdr = GetAuthHeader();
	if (!AuthHdr.IsEmpty())
	{
		HttpRequest->SetHeader(TEXT("Authorization"), AuthHdr);
	}
	HttpRequest->SetContentAsString(RequestBody);

	// If the HTTP POST itself fails (network error, non-2xx), reject the promise immediately.
	// On success the actual JSON-RPC response arrives via SSE, not via the POST response body.
	const int32 CapturedId = RequestId;

	HttpRequest->OnProcessRequestComplete().BindLambda(
		[WeakThis = AsWeak(), CapturedId](FHttpRequestPtr /*Req*/, FHttpResponsePtr Response, bool bSuccess)
		{
			// A 202 Accepted means the server queued the request; response via SSE.
			// Anything else (or a network failure) is a hard error.
			const bool bAccepted = bSuccess && Response.IsValid()
				&& (Response->GetResponseCode() == 202 || EHttpResponseCodes::IsOk(Response->GetResponseCode()));

			if (!bAccepted)
			{
				if (TSharedPtr<FMCPClientToolset> StrongThis = WeakThis.Pin())
				{
					if (TSharedPtr<TPromise<TSharedPtr<FJsonObject>>>* PromisePtr =
						StrongThis->PendingRequests.Find(CapturedId))
					{
						(*PromisePtr)->SetValue(nullptr);
						StrongThis->PendingRequests.Remove(CapturedId);
					}
				}
			}
		});

	HttpRequest->ProcessRequest();
	return Future;
}

FString FMCPClientToolset::BuildSchemaFromMCPTools(const TArray<TSharedPtr<FJsonValue>>& Tools)
{
	// Wrap the MCP tools array in the ToolsetRegistry envelope.
	// "description" is required by the backend; "version" is conventional.
	TSharedPtr<FJsonObject> Schema = MakeShared<FJsonObject>();
	Schema->SetStringField(TEXT("name"),        Config.Name);
	Schema->SetStringField(TEXT("description"), Config.Description.IsEmpty() ? Config.Name : Config.Description);
	Schema->SetStringField(TEXT("version"),     TEXT("1.0"));

	TArray<TSharedPtr<FJsonValue>> ToolsArray;
	for (const TSharedPtr<FJsonValue>& ToolValue : Tools)
	{
		const TSharedPtr<FJsonObject>* ToolObj = nullptr;
		if (!ToolValue.IsValid() || !ToolValue->TryGetObject(ToolObj) || !ToolObj)
		{
			continue;
		}

		// Shallow-copy the tool object so we can modify fields without
		// mutating the original (which may be referenced elsewhere).
		TSharedPtr<FJsonObject> Tool = MakeShared<FJsonObject>();
		Tool->Values = (*ToolObj)->Values;

		// Qualify the tool name as "ToolsetName.ToolName" so the registry can
		// route calls correctly (it splits on the last dot to find the toolset).
		FString UnqualifiedName;
		if (Tool->TryGetStringField(TEXT("name"), UnqualifiedName))
		{
			Tool->SetStringField(TEXT("name"), Config.Name + TEXT(".") + UnqualifiedName);
		}

		// Strip top-level tool fields the backend's Pydantic model marks as extra_forbidden.
		Tool->Values.Remove(TEXT("execution"));

		// Strip fields from inputSchema / outputSchema that the backend rejects.
		static const TCHAR* SchemaFields[] = { TEXT("inputSchema"), TEXT("outputSchema") };
		for (const TCHAR* FieldName : SchemaFields)
		{
			const TSharedPtr<FJsonObject>* SchemaPtr = nullptr;
			if (Tool->TryGetObjectField(FieldName, SchemaPtr) && SchemaPtr)
			{
				TSharedPtr<FJsonObject> Sanitized = MakeShared<FJsonObject>();
				Sanitized->Values = (*SchemaPtr)->Values;
				Sanitized->Values.Remove(TEXT("title"));
				Sanitized->Values.Remove(TEXT("$schema"));
				Sanitized->Values.Remove(TEXT("additionalProperties"));
				Tool->SetObjectField(FieldName, Sanitized);
			}
		}

		ToolsArray.Add(MakeShared<FJsonValueObject>(Tool));
	}
	Schema->SetArrayField(TEXT("tools"), ToolsArray);

	FString SchemaStr;
	TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&SchemaStr);
	FJsonSerializer::Serialize(Schema.ToSharedRef(), Writer);
	return SchemaStr;
}

FString FMCPClientToolset::MCPResultToString(TSharedPtr<FJsonObject> Result)
{
	// The backend requires tool_response.content.response to be a JSON object (dict).
	// Always return a serialized FJsonObject, never a bare scalar or plain string.

	if (!Result.IsValid())
	{
		return TEXT("{}");
	}

	// MCP content items: [{ "type": "text", "text": "..." }, ...]
	const TArray<TSharedPtr<FJsonValue>>* ContentArray = nullptr;
	if (!Result->TryGetArrayField(TEXT("content"), ContentArray) || !ContentArray)
	{
		// Result has no content array — serialize the whole result object as-is.
		FString FallbackStr;
		TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&FallbackStr);
		FJsonSerializer::Serialize(Result.ToSharedRef(), Writer);
		return FallbackStr;
	}

	TArray<FString> TextParts;
	for (const TSharedPtr<FJsonValue>& Item : *ContentArray)
	{
		const TSharedPtr<FJsonObject>* ItemObj = nullptr;
		if (!Item.IsValid() || !Item->TryGetObject(ItemObj) || !ItemObj)
		{
			continue;
		}

		FString Type;
		(*ItemObj)->TryGetStringField(TEXT("type"), Type);
		if (Type == TEXT("text"))
		{
			FString Text;
			if ((*ItemObj)->TryGetStringField(TEXT("text"), Text))
			{
				TextParts.Add(Text);
			}
		}
	}

	// Wrap the joined text in an object so the backend receives a dict.
	TSharedPtr<FJsonObject> Response = MakeShared<FJsonObject>();
	Response->SetStringField(TEXT("text"), FString::Join(TextParts, TEXT("\n")));

	FString ResponseStr;
	TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&ResponseStr);
	FJsonSerializer::Serialize(Response.ToSharedRef(), Writer);
	return ResponseStr;
}

void FMCPClientToolset::RejectAllPendingRequests()
{
	for (auto& Pair : PendingRequests)
	{
		if (Pair.Value.IsValid())
		{
			Pair.Value->SetValue(nullptr);
		}
	}
	PendingRequests.Empty();
}

// ---------------------------------------------------------------------------
// Streamable HTTP transport
// ---------------------------------------------------------------------------

void FMCPClientToolset::StartStreamableHTTPHandshake()
{
	UE_LOGF(LogMCPClientToolset, Log,
		"MCPClientToolset '%ls': starting Streamable HTTP handshake at %ls",
		*Config.Name, *Config.ServerUrl);

	TSharedPtr<FJsonObject> InitParams = MakeShared<FJsonObject>();
	InitParams->SetStringField(TEXT("protocolVersion"), TEXT("2024-11-05"));
	InitParams->SetObjectField(TEXT("capabilities"), MakeShared<FJsonObject>());

	TSharedPtr<FJsonObject> ClientInfo = MakeShared<FJsonObject>();
	ClientInfo->SetStringField(TEXT("name"),    TEXT("UE-ToolsetRegistry"));
	ClientInfo->SetStringField(TEXT("version"), TEXT("1.0"));
	InitParams->SetObjectField(TEXT("clientInfo"), ClientInfo);

	SendStreamableHTTPRequest(TEXT("initialize"), InitParams)
		.Next([WeakThis = AsWeak()](TSharedPtr<FJsonObject> InitResponse)
		{
			TSharedPtr<FMCPClientToolset> StrongThis = WeakThis.Pin();
			if (!StrongThis.IsValid())
			{
				return;
			}

			if (!InitResponse.IsValid())
			{
				if (StrongThis->CreatePromise.IsValid())
				{
					StrongThis->CreatePromise->SetValue(
						MakeError(FString::Printf(TEXT("MCP initialize failed for '%s'"), *StrongThis->Config.Name)));
					StrongThis->CreatePromise.Reset();
				}
				StrongThis->Self.Reset();
				return;
			}

			// Cache server version from result.serverInfo
			{
				const TSharedPtr<FJsonObject>* ResultObj = nullptr;
				if (InitResponse->TryGetObjectField(TEXT("result"), ResultObj) && ResultObj)
				{
					const TSharedPtr<FJsonObject>* ServerInfoObj = nullptr;
					if ((*ResultObj)->TryGetObjectField(TEXT("serverInfo"), ServerInfoObj) && ServerInfoObj)
					{
						(*ServerInfoObj)->TryGetStringField(TEXT("version"), StrongThis->ServerVersion);
					}
				}
			}

			// Fire-and-forget: notifications/initialized
			{
				TSharedPtr<FJsonObject> Notif = MakeShared<FJsonObject>();
				Notif->SetStringField(TEXT("jsonrpc"), TEXT("2.0"));
				Notif->SetStringField(TEXT("method"),  TEXT("notifications/initialized"));

				FString NotifBody;
				TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&NotifBody);
				FJsonSerializer::Serialize(Notif.ToSharedRef(), Writer);

				TSharedPtr<IHttpRequest, ESPMode::ThreadSafe> NotifRequest = FHttpModule::Get().CreateRequest();
				NotifRequest->SetURL(StrongThis->Config.ServerUrl);
				NotifRequest->SetVerb(TEXT("POST"));
				NotifRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
				NotifRequest->SetHeader(TEXT("Accept"), TEXT("application/json, text/event-stream"));

				const FString AuthHdr = StrongThis->GetAuthHeader();
				if (!AuthHdr.IsEmpty())
				{
					NotifRequest->SetHeader(TEXT("Authorization"), AuthHdr);
				}
				if (!StrongThis->StreamableSessionId.IsEmpty())
				{
					NotifRequest->SetHeader(TEXT("Mcp-Session-Id"), StrongThis->StreamableSessionId);
				}
				NotifRequest->SetContentAsString(NotifBody);
				NotifRequest->ProcessRequest();
			}

			// tools/list
			StrongThis->SendStreamableHTTPRequest(TEXT("tools/list"), nullptr)
				.Next([WeakThis](TSharedPtr<FJsonObject> ToolsResponse)
				{
					TSharedPtr<FMCPClientToolset> StrongThis2 = WeakThis.Pin();
					if (!StrongThis2.IsValid())
					{
						return;
					}

					if (!ToolsResponse.IsValid())
					{
						if (StrongThis2->CreatePromise.IsValid())
						{
							StrongThis2->CreatePromise->SetValue(
								MakeError(FString::Printf(TEXT("MCP tools/list failed for '%s'"), *StrongThis2->Config.Name)));
							StrongThis2->CreatePromise.Reset();
						}
						StrongThis2->Self.Reset();
						return;
					}

					// Extract tools array from result
					TArray<TSharedPtr<FJsonValue>> Tools;
					{
						const TSharedPtr<FJsonObject>* ResultObj = nullptr;
						if (ToolsResponse->TryGetObjectField(TEXT("result"), ResultObj) && ResultObj)
						{
							const TArray<TSharedPtr<FJsonValue>>* ToolsArray = nullptr;
							if ((*ResultObj)->TryGetArrayField(TEXT("tools"), ToolsArray) && ToolsArray)
							{
								Tools = *ToolsArray;
							}
						}
					}

					StrongThis2->CachedSchemaJson = StrongThis2->BuildSchemaFromMCPTools(Tools);

					UE_LOGF(LogMCPClientToolset, Log,
						"MCPClientToolset '%ls': registered %d tool(s)",
						*StrongThis2->Config.Name, Tools.Num());

					if (StrongThis2->CreatePromise.IsValid())
					{
						StrongThis2->CreatePromise->SetValue(MakeValue(StrongThis2));
						StrongThis2->CreatePromise.Reset();
					}
					StrongThis2->Self.Reset();
				});
		});
}

TFuture<TSharedPtr<FJsonObject>> FMCPClientToolset::SendStreamableHTTPRequest(
	const FString& Method, TSharedPtr<FJsonObject> Params)
{
	auto Promise = MakeShared<TPromise<TSharedPtr<FJsonObject>>>();
	TFuture<TSharedPtr<FJsonObject>> Future = Promise->GetFuture();

	const int32 RequestId = NextRequestId++;

	// Build JSON-RPC 2.0 request body
	TSharedPtr<FJsonObject> JsonRpc = MakeShared<FJsonObject>();
	JsonRpc->SetStringField(TEXT("jsonrpc"), TEXT("2.0"));
	JsonRpc->SetNumberField(TEXT("id"), RequestId);
	JsonRpc->SetStringField(TEXT("method"), Method);
	if (Params.IsValid())
	{
		JsonRpc->SetObjectField(TEXT("params"), Params);
	}

	FString RequestBody;
	{
		TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&RequestBody);
		FJsonSerializer::Serialize(JsonRpc.ToSharedRef(), Writer);
	}

	PendingRequests.Add(RequestId, Promise);

	// Per-request SPSC queue for the streaming response body
	TSharedPtr<FSSEStreamState, ESPMode::ThreadSafe> StreamState =
		MakeShared<FSSEStreamState, ESPMode::ThreadSafe>();

	UE_LOGF(LogMCPClientToolset, Log, "MCPClientToolset '%ls': SendStreamableHTTPRequest URL='%ls' len=%d method=%ls",
		*Config.Name, *Config.ServerUrl, Config.ServerUrl.Len(), *Method);

	TSharedPtr<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();
	HttpRequest->SetURL(Config.ServerUrl);
	HttpRequest->SetVerb(TEXT("POST"));
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
	HttpRequest->SetHeader(TEXT("Accept"), TEXT("application/json, text/event-stream"));

	const FString AuthHdr = GetAuthHeader();
	if (!AuthHdr.IsEmpty())
	{
		HttpRequest->SetHeader(TEXT("Authorization"), AuthHdr);
	}
	if (!StreamableSessionId.IsEmpty())
	{
		HttpRequest->SetHeader(TEXT("Mcp-Session-Id"), StreamableSessionId);
	}
	HttpRequest->SetContentAsString(RequestBody);

	// Stream delegate — HTTP thread enqueues, game thread drains in OnProcessRequestComplete
	TSharedPtr<FSSEStreamState, ESPMode::ThreadSafe> CapturedStreamState = StreamState;
	HttpRequest->SetResponseBodyReceiveStreamDelegateV2(
		FHttpRequestStreamDelegateV2::CreateLambda(
			[CapturedStreamState](void* Ptr, int64& InOutLength)
			{
				if (CapturedStreamState.IsValid() && InOutLength > 0)
				{
					TArray<uint8> Chunk;
					Chunk.Append(static_cast<const uint8*>(Ptr), static_cast<int32>(InOutLength));
					CapturedStreamState->Queue.Enqueue(MoveTemp(Chunk));
				}
			}));

	const int32 CapturedId = RequestId;

	HttpRequest->OnProcessRequestComplete().BindLambda(
		[WeakThis = AsWeak(), CapturedStreamState, CapturedId]
		(FHttpRequestPtr /*Req*/, FHttpResponsePtr Response, bool bSuccess)
		{
			TSharedPtr<FMCPClientToolset> StrongThis = WeakThis.Pin();
			if (!StrongThis.IsValid())
			{
				return;
			}

			// Drain the SPSC queue into a single buffer
			TArray<uint8> AllBytes;
			if (CapturedStreamState.IsValid())
			{
				TArray<uint8> Chunk;
				while (CapturedStreamState->Queue.Dequeue(Chunk))
				{
					AllBytes.Append(Chunk);
				}
			}

			// Handle failure
			if (!bSuccess || !Response.IsValid() || !EHttpResponseCodes::IsOk(Response->GetResponseCode()))
			{
				if (TSharedPtr<TPromise<TSharedPtr<FJsonObject>>>* PromisePtr =
					StrongThis->PendingRequests.Find(CapturedId))
				{
					(*PromisePtr)->SetValue(nullptr);
					StrongThis->PendingRequests.Remove(CapturedId);
				}
				return;
			}

			// Capture session ID from the first successful response
			if (StrongThis->StreamableSessionId.IsEmpty())
			{
				FString SessionId = Response->GetHeader(TEXT("Mcp-Session-Id"));
				if (SessionId.IsEmpty())
				{
					SessionId = Response->GetHeader(TEXT("mcp-session-id"));
				}
				if (!SessionId.IsEmpty())
				{
					StrongThis->StreamableSessionId = SessionId;
				}
			}

			const FString ContentType = Response->GetHeader(TEXT("Content-Type"));
			TSharedPtr<FJsonObject> ParsedResponse;

			if (ContentType.Contains(TEXT("application/json")))
			{
				if (AllBytes.Num() > 0)
				{
					AllBytes.Add(0); // null-terminate for UTF8_TO_TCHAR
					const FString JsonStr = UTF8_TO_TCHAR(reinterpret_cast<const ANSICHAR*>(AllBytes.GetData()));
					TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(JsonStr);
					FJsonSerializer::Deserialize(Reader, ParsedResponse);
				}
			}
			else
			{
				// text/event-stream — parse SSE events inline to find our response
				if (AllBytes.Num() > 0)
				{
					AllBytes.Add(0);
					FString Buffer = UTF8_TO_TCHAR(reinterpret_cast<const ANSICHAR*>(AllBytes.GetData()));

					while (true)
					{
						const int32 LFIdx   = Buffer.Find(TEXT("\n\n"),     ESearchCase::CaseSensitive);
						const int32 CRLFIdx = Buffer.Find(TEXT("\r\n\r\n"), ESearchCase::CaseSensitive);

						int32 DelimIdx    = INDEX_NONE;
						int32 DelimLength = 0;

						if (LFIdx != INDEX_NONE && (CRLFIdx == INDEX_NONE || LFIdx <= CRLFIdx))
						{
							DelimIdx    = LFIdx;
							DelimLength = 2;
						}
						else if (CRLFIdx != INDEX_NONE)
						{
							DelimIdx    = CRLFIdx;
							DelimLength = 4;
						}

						if (DelimIdx == INDEX_NONE)
						{
							break;
						}

						const FString EventBlock = Buffer.Left(DelimIdx);
						Buffer = Buffer.Mid(DelimIdx + DelimLength);

						FString EventType = TEXT("message");
						FString EventData;

						TArray<FString> Lines;
						EventBlock.ParseIntoArrayLines(Lines, /*bCullEmpty=*/false);

						for (const FString& Line : Lines)
						{
							if (Line.StartsWith(TEXT("event:"), ESearchCase::CaseSensitive))
							{
								EventType = Line.Mid(6).TrimStartAndEnd();
							}
							else if (Line.StartsWith(TEXT("data:"), ESearchCase::CaseSensitive))
							{
								EventData = Line.Mid(5).TrimStartAndEnd();
							}
						}

						if (EventType == TEXT("message") && !EventData.IsEmpty())
						{
							TSharedPtr<FJsonObject> EventJson;
							TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(EventData);
							if (FJsonSerializer::Deserialize(Reader, EventJson) && EventJson.IsValid())
							{
								int32 EventId = -1;
								double IdDouble = -1.0;
								if (EventJson->TryGetNumberField(TEXT("id"), IdDouble))
								{
									EventId = static_cast<int32>(IdDouble);
								}
								else
								{
									FString IdStr;
									if (EventJson->TryGetStringField(TEXT("id"), IdStr))
									{
										EventId = FCString::Atoi(*IdStr);
									}
								}

								if (EventId == CapturedId)
								{
									ParsedResponse = EventJson;
									break;
								}
							}
						}
					}
				}
			}

			if (TSharedPtr<TPromise<TSharedPtr<FJsonObject>>>* PromisePtr =
				StrongThis->PendingRequests.Find(CapturedId))
			{
				(*PromisePtr)->SetValue(ParsedResponse);
				StrongThis->PendingRequests.Remove(CapturedId);
			}
		});

	HttpRequest->ProcessRequest();
	return Future;
}

// ---------------------------------------------------------------------------
// OAuth 2.0 Authorization Code + PKCE
// ---------------------------------------------------------------------------

void FMCPClientToolset::StartOAuthFlow()
{
	OAuthState = EOAuthState::Discovering;

	// RFC 8414: metadata lives at {origin}/.well-known/oauth-authorization-server,
	// where origin = scheme + "://" + host (+ optional port), with no path component.
	FString Origin = Config.ServerUrl;
	{
		const int32 SchemeEnd = Origin.Find(TEXT("://"), ESearchCase::CaseSensitive);
		if (SchemeEnd != INDEX_NONE)
		{
			const int32 PathStart = Origin.Find(TEXT("/"), ESearchCase::CaseSensitive,
				ESearchDir::FromStart, SchemeEnd + 3);
			if (PathStart != INDEX_NONE)
			{
				Origin = Origin.Left(PathStart);
			}
		}
	}
	const FString DiscoveryUrl = Origin + TEXT("/.well-known/oauth-authorization-server");
	UE_LOGF(LogMCPClientToolset, Log,
		"MCPClientToolset '%ls': starting OAuth 2.0 discovery at %ls",
		*Config.Name, *DiscoveryUrl);

	TSharedPtr<IHttpRequest, ESPMode::ThreadSafe> Request = FHttpModule::Get().CreateRequest();
	Request->SetURL(DiscoveryUrl);
	Request->SetVerb(TEXT("GET"));

	Request->OnProcessRequestComplete().BindLambda(
		[WeakThis = AsWeak()](FHttpRequestPtr Req, FHttpResponsePtr Response, bool bSuccess)
		{
			if (TSharedPtr<FMCPClientToolset> StrongThis = WeakThis.Pin())
			{
				StrongThis->OnOAuthDiscoveryComplete(Req, Response, bSuccess);
			}
		});
	Request->ProcessRequest();
}

void FMCPClientToolset::OnOAuthDiscoveryComplete(
	FHttpRequestPtr /*Req*/, FHttpResponsePtr Response, bool bSuccess)
{
	if (!bSuccess || !Response.IsValid() || !EHttpResponseCodes::IsOk(Response->GetResponseCode()))
	{
		FailOAuth(FString::Printf(TEXT("OAuth discovery request failed for '%s'"), *Config.Name));
		return;
	}

	TSharedPtr<FJsonObject> Json;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Response->GetContentAsString());
	if (!FJsonSerializer::Deserialize(Reader, Json) || !Json.IsValid())
	{
		FailOAuth(FString::Printf(TEXT("OAuth discovery JSON parse failed for '%s'"), *Config.Name));
		return;
	}

	if (!Json->TryGetStringField(TEXT("authorization_endpoint"), OAuthAuthEndpoint) ||
		!Json->TryGetStringField(TEXT("token_endpoint"), OAuthTokenEndpoint))
	{
		FailOAuth(FString::Printf(TEXT("OAuth discovery missing endpoints for '%s'"), *Config.Name));
		return;
	}

	// Optional: RFC 7591 dynamic client registration endpoint
	Json->TryGetStringField(TEXT("registration_endpoint"), OAuthRegistrationEndpoint);

	// Load any previously-saved tokens (access token, refresh token, cached client ID).
	LoadTokensFromConfig();

	// If we have a non-expired access token, skip the browser entirely.
	if (!AccessToken.IsEmpty())
	{
		UE_LOGF(LogMCPClientToolset, Log,
			"MCPClientToolset '%ls': using cached OAuth access token", *Config.Name);
		OAuthState = EOAuthState::Complete;
		if (Config.bStreamableHTTP)
			StartStreamableHTTPHandshake();
		else
			StartSSEConnection();
		return;
	}

	// If we have a refresh token, attempt a silent refresh before opening a browser.
	if (!RefreshToken.IsEmpty())
	{
		TryRefreshToken();
		return;
	}

	// No cached credentials — proceed with the full browser flow.
	AllocateOAuthPort();
	if (OAuthLocalPort == 0)
	{
		FailOAuth(TEXT("No available local port for OAuth callback listener"));
		return;
	}

	// If no client ID was configured, attempt dynamic registration (RFC 7591).
	// If the server doesn't advertise a registration endpoint, a client ID is required.
	if (Config.OAuthClientId.IsEmpty())
	{
		if (!OAuthRegistrationEndpoint.IsEmpty())
		{
			RegisterOAuthClient();
		}
		else
		{
			FailOAuth(FString::Printf(
				TEXT("OAuth client ID not set and server '%s' does not advertise a registration endpoint"),
				*Config.Name));
		}
		return;
	}

	LaunchOAuthBrowser();
}

void FMCPClientToolset::RegisterOAuthClient()
{
	const FString RedirectUri = FString::Printf(TEXT("http://localhost:%d/callback"), OAuthLocalPort);

	TSharedPtr<FJsonObject> RegBody = MakeShared<FJsonObject>();
	RegBody->SetStringField(TEXT("client_name"), TEXT("UE-ToolsetRegistry"));
	RegBody->SetStringField(TEXT("token_endpoint_auth_method"), TEXT("none")); // public client

	TArray<TSharedPtr<FJsonValue>> RedirectUris;
	RedirectUris.Add(MakeShared<FJsonValueString>(RedirectUri));
	RegBody->SetArrayField(TEXT("redirect_uris"), RedirectUris);

	TArray<TSharedPtr<FJsonValue>> GrantTypes;
	GrantTypes.Add(MakeShared<FJsonValueString>(TEXT("authorization_code")));
	RegBody->SetArrayField(TEXT("grant_types"), GrantTypes);

	TArray<TSharedPtr<FJsonValue>> ResponseTypes;
	ResponseTypes.Add(MakeShared<FJsonValueString>(TEXT("code")));
	RegBody->SetArrayField(TEXT("response_types"), ResponseTypes);

	FString RegBodyStr;
	TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&RegBodyStr);
	FJsonSerializer::Serialize(RegBody.ToSharedRef(), Writer);

	UE_LOGF(LogMCPClientToolset, Log,
		"MCPClientToolset '%ls': registering OAuth client at %ls",
		*Config.Name, *OAuthRegistrationEndpoint);

	TSharedPtr<IHttpRequest, ESPMode::ThreadSafe> Request = FHttpModule::Get().CreateRequest();
	Request->SetURL(OAuthRegistrationEndpoint);
	Request->SetVerb(TEXT("POST"));
	Request->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
	Request->SetContentAsString(RegBodyStr);

	Request->OnProcessRequestComplete().BindLambda(
		[WeakThis = AsWeak()](FHttpRequestPtr Req, FHttpResponsePtr Response, bool bSuccess)
		{
			if (TSharedPtr<FMCPClientToolset> StrongThis = WeakThis.Pin())
			{
				StrongThis->OnOAuthClientRegistered(Req, Response, bSuccess);
			}
		});
	Request->ProcessRequest();
}

void FMCPClientToolset::OnOAuthClientRegistered(
	FHttpRequestPtr /*Req*/, FHttpResponsePtr Response, bool bSuccess)
{
	if (!bSuccess || !Response.IsValid() || !EHttpResponseCodes::IsOk(Response->GetResponseCode()))
	{
		FailOAuth(FString::Printf(TEXT("OAuth dynamic client registration failed for '%s'"), *Config.Name));
		return;
	}

	TSharedPtr<FJsonObject> Json;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Response->GetContentAsString());
	if (!FJsonSerializer::Deserialize(Reader, Json) || !Json.IsValid())
	{
		FailOAuth(FString::Printf(TEXT("OAuth registration response parse failed for '%s'"), *Config.Name));
		return;
	}

	FString NewClientId;
	if (!Json->TryGetStringField(TEXT("client_id"), NewClientId) || NewClientId.IsEmpty())
	{
		FailOAuth(FString::Printf(TEXT("OAuth registration response missing client_id for '%s'"), *Config.Name));
		return;
	}

	Config.OAuthClientId = NewClientId;
	UE_LOGF(LogMCPClientToolset, Log,
		"MCPClientToolset '%ls': dynamic client registration succeeded (client_id=%ls)",
		*Config.Name, *Config.OAuthClientId);

	// Persist the client ID so we skip re-registration on future editor runs.
	SaveTokensToConfig();

	LaunchOAuthBrowser();
}

void FMCPClientToolset::LaunchOAuthBrowser()
{
	// 1. Generate code_verifier: 43 random chars from [A-Za-z0-9-._~]
	static const TCHAR Charset[] =
		TEXT("ABCDEFGHIJKLMNOPQRSTUVWXYZabcdefghijklmnopqrstuvwxyz0123456789-._~");
	constexpr int32 CharsetLen = UE_ARRAY_COUNT(Charset) - 1; // exclude null terminator

	OAuthCodeVerifier.Reset();
	OAuthCodeVerifier.Reserve(43);
	for (int32 i = 0; i < 43; ++i)
	{
		OAuthCodeVerifier += Charset[FMath::RandRange(0, CharsetLen - 1)];
	}

	// 2. SHA-256(UTF-8(code_verifier))
	uint8 HashBytes[32];
	{
		FTCHARToUTF8 Utf8(*OAuthCodeVerifier);
		ComputeSHA256(reinterpret_cast<const uint8*>(Utf8.Get()), static_cast<uint32>(Utf8.Length()), HashBytes);
	}

	// 3. code_challenge = Base64URL(hash), no padding
	FString CodeChallenge;
	{
		TArray<uint8> HashArray(HashBytes, 32);
		CodeChallenge = FBase64::Encode(HashArray, EBase64Mode::UrlSafe);
		while (CodeChallenge.EndsWith(TEXT("=")))
		{
			CodeChallenge.LeftChopInline(1);
		}
	}

	// 4. state = 16 random hex chars (CSRF protection)
	OAuthCSRFState = FGuid::NewGuid().ToString(EGuidFormats::Digits).Left(16).ToLower();

	// 5. Bind GET /callback and start the listener
	// (OAuthLocalPort and OAuthRouter were set in OnOAuthDiscoveryComplete)
	OAuthRouteHandle = OAuthRouter->BindRoute(
		FHttpPath(TEXT("/callback")),
		EHttpServerRequestVerbs::VERB_GET,
		FHttpRequestHandler::CreateLambda(
			[WeakThis = AsWeak()](const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete) -> bool
			{
				if (TSharedPtr<FMCPClientToolset> StrongThis = WeakThis.Pin())
				{
					return StrongThis->HandleOAuthCallback(Request, OnComplete);
				}
				return false;
			}));

	FHttpServerModule::Get().StartAllListeners();
	OAuthState = EOAuthState::WaitingForCode;

	// 7. Build the authorization URL
	const FString RedirectUri = FString::Printf(TEXT("http://localhost:%d/callback"), OAuthLocalPort);
	const FString AuthUrl = FString::Printf(
		TEXT("%s?response_type=code&client_id=%s&redirect_uri=%s&scope=%s&state=%s&code_challenge=%s&code_challenge_method=S256"),
		*OAuthAuthEndpoint,
		*FPlatformHttp::UrlEncode(Config.OAuthClientId),
		*FPlatformHttp::UrlEncode(RedirectUri),
		*FPlatformHttp::UrlEncode(Config.OAuthScope),
		*OAuthCSRFState,
		*CodeChallenge);

	// 8. Open the browser
	FPlatformProcess::LaunchURL(*AuthUrl, nullptr, nullptr);

	UE_LOGF(LogMCPClientToolset, Log,
		"MCPClientToolset '%ls': browser opened — waiting for user authorization at http://localhost:%d/callback",
		*Config.Name, OAuthLocalPort);

	// 9. Schedule a 5-minute timeout
	OAuthTimeoutHandle = FTSTicker::GetCoreTicker().AddTicker(
		FTickerDelegate::CreateSP(AsShared(), &FMCPClientToolset::OnOAuthTimeout),
		300.0f);
}

bool FMCPClientToolset::HandleOAuthCallback(
	const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	// Extract code and state from query parameters
	const FString* CodePtr  = Request.QueryParams.Find(TEXT("code"));
	const FString* StatePtr = Request.QueryParams.Find(TEXT("state"));

	if (!CodePtr || CodePtr->IsEmpty() || !StatePtr || StatePtr->IsEmpty())
	{
		TUniquePtr<FHttpServerResponse> ErrorResp =
			FHttpServerResponse::Create(FString(TEXT("Missing code or state parameter")), TEXT("text/plain"));
		ErrorResp->Code = EHttpServerResponseCodes::BadRequest;
		OnComplete(MoveTemp(ErrorResp));
		FailOAuth(FString::Printf(TEXT("OAuth callback missing parameters for '%s'"), *Config.Name));
		return true;
	}

	if (*StatePtr != OAuthCSRFState)
	{
		TUniquePtr<FHttpServerResponse> ErrorResp =
			FHttpServerResponse::Create(FString(TEXT("OAuth state mismatch")), TEXT("text/plain"));
		ErrorResp->Code = EHttpServerResponseCodes::BadRequest;
		OnComplete(MoveTemp(ErrorResp));
		FailOAuth(FString::Printf(TEXT("OAuth state mismatch for '%s'"), *Config.Name));
		return true;
	}

	// Send the success page before cleaning up
	TUniquePtr<FHttpServerResponse> SuccessResp = FHttpServerResponse::Create(
		FString(TEXT("<html><body><h1>Authorization complete</h1><p>You may close this tab.</p></body></html>")),
		TEXT("text/html"));
	OnComplete(MoveTemp(SuccessResp));

	// Clean up our route binding; leave the port registered in the module
	// so other systems' listeners on other ports are unaffected.
	if (OAuthRouter.IsValid())
	{
		if (OAuthRouteHandle.IsValid())
		{
			OAuthRouter->UnbindRoute(OAuthRouteHandle);
		}
		OAuthRouter.Reset();
	}
	FTSTicker::GetCoreTicker().RemoveTicker(OAuthTimeoutHandle);

	// Proceed with code exchange
	const FString Code = *CodePtr;
	ExchangeOAuthCode(Code);
	return true;
}

void FMCPClientToolset::ExchangeOAuthCode(const FString& Code)
{
	OAuthState = EOAuthState::Exchanging;

	const FString RedirectUri = FString::Printf(TEXT("http://localhost:%d/callback"), OAuthLocalPort);

	const FString PostBody = FString::Printf(
		TEXT("grant_type=authorization_code&code=%s&redirect_uri=%s&client_id=%s&code_verifier=%s"),
		*FPlatformHttp::UrlEncode(Code),
		*FPlatformHttp::UrlEncode(RedirectUri),
		*FPlatformHttp::UrlEncode(Config.OAuthClientId),
		*FPlatformHttp::UrlEncode(OAuthCodeVerifier));

	TSharedPtr<IHttpRequest, ESPMode::ThreadSafe> Request = FHttpModule::Get().CreateRequest();
	Request->SetURL(OAuthTokenEndpoint);
	Request->SetVerb(TEXT("POST"));
	Request->SetHeader(TEXT("Content-Type"), TEXT("application/x-www-form-urlencoded"));
	Request->SetContentAsString(PostBody);

	Request->OnProcessRequestComplete().BindLambda(
		[WeakThis = AsWeak()](FHttpRequestPtr Req, FHttpResponsePtr Response, bool bSuccess)
		{
			if (TSharedPtr<FMCPClientToolset> StrongThis = WeakThis.Pin())
			{
				StrongThis->OnOAuthTokenExchangeComplete(Req, Response, bSuccess);
			}
		});
	Request->ProcessRequest();
}

void FMCPClientToolset::OnOAuthTokenExchangeComplete(
	FHttpRequestPtr /*Req*/, FHttpResponsePtr Response, bool bSuccess)
{
	if (!bSuccess || !Response.IsValid() || !EHttpResponseCodes::IsOk(Response->GetResponseCode()))
	{
		FailOAuth(FString::Printf(TEXT("OAuth token exchange failed for '%s'"), *Config.Name));
		return;
	}

	TSharedPtr<FJsonObject> Json;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Response->GetContentAsString());
	if (!FJsonSerializer::Deserialize(Reader, Json) || !Json.IsValid())
	{
		FailOAuth(FString::Printf(TEXT("OAuth token JSON parse failed for '%s'"), *Config.Name));
		return;
	}

	FString NewAccessToken;
	if (!Json->TryGetStringField(TEXT("access_token"), NewAccessToken) || NewAccessToken.IsEmpty())
	{
		FailOAuth(FString::Printf(TEXT("OAuth response missing access_token for '%s'"), *Config.Name));
		return;
	}

	AccessToken = NewAccessToken.TrimStartAndEnd();
	OAuthState  = EOAuthState::Complete;

	FString NewRefreshToken;
	if (Json->TryGetStringField(TEXT("refresh_token"), NewRefreshToken) && !NewRefreshToken.IsEmpty())
	{
		RefreshToken = NewRefreshToken.TrimStartAndEnd();
	}

	int32 ExpiresIn = 0;
	if (Json->TryGetNumberField(TEXT("expires_in"), ExpiresIn) && ExpiresIn > 0)
	{
		// Subtract 60s as a buffer so we refresh slightly before the token actually expires.
		TokenExpiresAt = FDateTime::UtcNow() + FTimespan::FromSeconds(ExpiresIn - 60);
	}

	SaveTokensToConfig();

	UE_LOGF(LogMCPClientToolset, Log,
		"MCPClientToolset '%ls': OAuth token obtained", *Config.Name);

	// Proceed with the MCP handshake now that we have a valid token
	if (Config.bStreamableHTTP)
	{
		StartStreamableHTTPHandshake();
	}
	else
	{
		StartSSEConnection();
	}
}

bool FMCPClientToolset::OnOAuthTimeout(float /*DeltaTime*/)
{
	FailOAuth(FString::Printf(TEXT("OAuth authorization timed out for '%s'"), *Config.Name));
	return false; // do not reschedule
}

void FMCPClientToolset::AllocateOAuthPort()
{
	for (int32 Port = 49152; Port <= 49162; ++Port)
	{
		TSharedPtr<IHttpRouter> Router = FHttpServerModule::Get().GetHttpRouter(Port);
		if (Router.IsValid())
		{
			OAuthRouter    = Router;
			OAuthLocalPort = Port;
			return;
		}
	}
}

// ---------------------------------------------------------------------------
// OAuth token persistence
// ---------------------------------------------------------------------------

static FString GetOAuthConfigSection(const FString& ServerName)
{
	FString Sanitized = ServerName;
	for (TCHAR& Ch : Sanitized)
	{
		if (!FChar::IsAlnum(Ch))
		{
			Ch = TEXT('_');
		}
	}
	return TEXT("MCPClientToolset_OAuth_") + Sanitized;
}

void FMCPClientToolset::SaveTokensToConfig()
{
	if (!GConfig)
	{
		return;
	}
	const FString Section = GetOAuthConfigSection(Config.Name);

	// Always write every field so stale values from a previous session are overwritten.
	GConfig->SetString(*Section, TEXT("ClientId"),     *Config.OAuthClientId,        GEditorPerProjectIni);
	GConfig->SetString(*Section, TEXT("AccessToken"),  *AccessToken,                 GEditorPerProjectIni);
	GConfig->SetString(*Section, TEXT("RefreshToken"), *RefreshToken,                GEditorPerProjectIni);
	GConfig->SetString(*Section, TEXT("ExpiresAt"),    *TokenExpiresAt.ToString(),   GEditorPerProjectIni);
	GConfig->Flush(false, GEditorPerProjectIni);
}

void FMCPClientToolset::LoadTokensFromConfig()
{
	if (!GConfig)
	{
		return;
	}
	const FString Section = GetOAuthConfigSection(Config.Name);

	// Load the dynamically-registered client ID if none was provided in the settings.
	if (Config.OAuthClientId.IsEmpty())
	{
		FString CachedClientId;
		GConfig->GetString(*Section, TEXT("ClientId"), CachedClientId, GEditorPerProjectIni);
		Config.OAuthClientId = CachedClientId;
	}

	FString StoredAccessToken;
	GConfig->GetString(*Section, TEXT("AccessToken"),  StoredAccessToken, GEditorPerProjectIni);
	GConfig->GetString(*Section, TEXT("RefreshToken"), RefreshToken,      GEditorPerProjectIni);

	FString ExpiresAtStr;
	if (GConfig->GetString(*Section, TEXT("ExpiresAt"), ExpiresAtStr, GEditorPerProjectIni))
	{
		FDateTime::Parse(ExpiresAtStr, TokenExpiresAt);
	}

	// Only restore the access token if it hasn't expired.
	if (!StoredAccessToken.IsEmpty() && FDateTime::UtcNow() < TokenExpiresAt)
	{
		AccessToken = StoredAccessToken;
	}
}

// ---------------------------------------------------------------------------
// OAuth silent token refresh
// ---------------------------------------------------------------------------

void FMCPClientToolset::TryRefreshToken()
{
	UE_LOGF(LogMCPClientToolset, Log,
		"MCPClientToolset '%ls': attempting silent OAuth token refresh", *Config.Name);

	const FString Body = FString::Printf(
		TEXT("grant_type=refresh_token&refresh_token=%s&client_id=%s"),
		*FPlatformHttp::UrlEncode(RefreshToken),
		*FPlatformHttp::UrlEncode(Config.OAuthClientId));

	TSharedPtr<IHttpRequest, ESPMode::ThreadSafe> Request = FHttpModule::Get().CreateRequest();
	Request->SetURL(OAuthTokenEndpoint);
	Request->SetVerb(TEXT("POST"));
	Request->SetHeader(TEXT("Content-Type"), TEXT("application/x-www-form-urlencoded"));
	Request->SetContentAsString(Body);

	Request->OnProcessRequestComplete().BindLambda(
		[WeakThis = AsWeak()](FHttpRequestPtr Req, FHttpResponsePtr Response, bool bSuccess)
		{
			if (TSharedPtr<FMCPClientToolset> StrongThis = WeakThis.Pin())
			{
				StrongThis->OnRefreshTokenComplete(Req, Response, bSuccess);
			}
		});
	Request->ProcessRequest();
}

void FMCPClientToolset::OnRefreshTokenComplete(
	FHttpRequestPtr /*Req*/, FHttpResponsePtr Response, bool bSuccess)
{
	// On any failure, clear the stale refresh token and fall back to the full browser flow.
	auto FallBackToBrowser = [this]()
	{
		RefreshToken.Reset();
		SaveTokensToConfig();

		AllocateOAuthPort();
		if (OAuthLocalPort == 0)
		{
			FailOAuth(TEXT("No available local port for OAuth callback listener"));
			return;
		}
		if (Config.OAuthClientId.IsEmpty())
		{
			if (!OAuthRegistrationEndpoint.IsEmpty())
				RegisterOAuthClient();
			else
				FailOAuth(FString::Printf(
					TEXT("OAuth client ID not set and server '%s' does not advertise a registration endpoint"),
					*Config.Name));
		}
		else
		{
			LaunchOAuthBrowser();
		}
	};

	if (!bSuccess || !Response.IsValid() || !EHttpResponseCodes::IsOk(Response->GetResponseCode()))
	{
		UE_LOGF(LogMCPClientToolset, Log,
			"MCPClientToolset '%ls': token refresh failed (HTTP %d), falling back to browser login",
			*Config.Name, Response.IsValid() ? Response->GetResponseCode() : 0);
		FallBackToBrowser();
		return;
	}

	TSharedPtr<FJsonObject> Json;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Response->GetContentAsString());
	if (!FJsonSerializer::Deserialize(Reader, Json) || !Json.IsValid())
	{
		UE_LOGF(LogMCPClientToolset, Log,
			"MCPClientToolset '%ls': token refresh response parse failed, falling back to browser login",
			*Config.Name);
		FallBackToBrowser();
		return;
	}

	FString NewAccessToken;
	if (!Json->TryGetStringField(TEXT("access_token"), NewAccessToken) || NewAccessToken.IsEmpty())
	{
		UE_LOGF(LogMCPClientToolset, Log,
			"MCPClientToolset '%ls': token refresh response missing access_token, falling back to browser login",
			*Config.Name);
		FallBackToBrowser();
		return;
	}

	AccessToken = NewAccessToken.TrimStartAndEnd();
	OAuthState  = EOAuthState::Complete;

	FString NewRefreshToken;
	if (Json->TryGetStringField(TEXT("refresh_token"), NewRefreshToken) && !NewRefreshToken.IsEmpty())
	{
		RefreshToken = NewRefreshToken.TrimStartAndEnd();
	}

	int32 ExpiresIn = 0;
	if (Json->TryGetNumberField(TEXT("expires_in"), ExpiresIn) && ExpiresIn > 0)
	{
		TokenExpiresAt = FDateTime::UtcNow() + FTimespan::FromSeconds(ExpiresIn - 60);
	}

	SaveTokensToConfig();

	UE_LOGF(LogMCPClientToolset, Log,
		"MCPClientToolset '%ls': OAuth token refreshed silently", *Config.Name);

	if (Config.bStreamableHTTP)
		StartStreamableHTTPHandshake();
	else
		StartSSEConnection();
}

void FMCPClientToolset::FailOAuth(const FString& Reason)
{
	UE_LOGF(LogMCPClientToolset, Warning,
		"MCPClientToolset: OAuth failed — %ls", *Reason);

	FTSTicker::GetCoreTicker().RemoveTicker(OAuthTimeoutHandle);

	if (OAuthRouter.IsValid())
	{
		if (OAuthRouteHandle.IsValid())
		{
			OAuthRouter->UnbindRoute(OAuthRouteHandle);
		}
		OAuthRouter.Reset();
	}

	OAuthState = EOAuthState::Idle;

	if (CreatePromise.IsValid())
	{
		CreatePromise->SetValue(MakeError(Reason));
		CreatePromise.Reset();
	}
	Self.Reset();
}

} // namespace UE::ToolsetRegistry
