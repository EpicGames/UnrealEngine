// Copyright Epic Games, Inc. All Rights Reserved.

#include "MCPClientToolset/MCPClientToolsetSubsystem.h"

#include "MCPClientToolset/MCPClientToolset.h"
#include "MCPClientToolset/MCPToolsetSettings.h"
#include "Module.h"
#include "Subsystems/SubsystemCollection.h"
#include "ToolsetRegistry/ToolsetRegistrySubsystem.h"

void UMCPClientToolsetSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	// Ensure ToolsetRegistrySubsystem is initialized before we try to register into it.
	Collection.InitializeDependency<UToolsetRegistrySubsystem>();

	const UMCPToolsetSettings* Settings = GetDefault<UMCPToolsetSettings>();
	if (!Settings)
	{
		return;
	}

	for (const FMCPServerConfig& ServerConfig : Settings->MCPServers)
	{
		if (!ServerConfig.bEnabled || ServerConfig.Name.IsEmpty() || ServerConfig.ServerUrl.IsEmpty())
		{
			continue;
		}

		UE::ToolsetRegistry::FMCPClientToolset::FConfig MCPConfig;
		MCPConfig.Name            = ServerConfig.Name;
		MCPConfig.Description     = ServerConfig.Description;
		MCPConfig.ServerUrl       = ServerConfig.ServerUrl;
		MCPConfig.ApiKey          = ServerConfig.ApiKey;
		MCPConfig.bStreamableHTTP = (ServerConfig.Transport == EMCPTransport::StreamableHTTP);
		MCPConfig.bOAuth          = (ServerConfig.Auth      == EMCPAuth::OAuth2);
		MCPConfig.OAuthClientId   = ServerConfig.OAuthClientId;
		MCPConfig.OAuthScope      = ServerConfig.OAuthScope;

		TWeakObjectPtr<UMCPClientToolsetSubsystem> WeakThis(this);

		UE::ToolsetRegistry::FMCPClientToolset::Create(MCPConfig)
			.Next([WeakThis](TValueOrError<TSharedPtr<UE::ToolsetRegistry::FMCPClientToolset>, FString> Result)
			{
				UMCPClientToolsetSubsystem* StrongThis = WeakThis.Get();
				if (!StrongThis)
				{
					return;
				}

				if (Result.HasError())
				{
					UE_LOGF(LogMCPClientToolset, Warning,
						"MCPClientToolset: failed to connect — %ls", *Result.GetError());
					return;
				}

				TSharedPtr<UE::ToolsetRegistry::FMCPClientToolset> Toolset = Result.GetValue();

				auto RegistryResult = UToolsetRegistrySubsystem::Get();
				if (RegistryResult.HasError())
				{
					UE_LOGF(LogMCPClientToolset, Warning,
						"MCPClientToolset: could not register '%ls' — %ls",
						*Toolset->GetToolsetName(), *RegistryResult.GetError());
					return;
				}

				StrongThis->Toolsets.Add(Toolset);
				RegistryResult.GetValue()->ToolsetRegistry.RegisterToolset(Toolset);
				UE_LOGF(LogMCPClientToolset, Log,
					"MCPClientToolset: registered '%ls'", *Toolset->GetToolsetName());
			});
	}
}

void UMCPClientToolsetSubsystem::Deinitialize()
{
	auto RegistryResult = UToolsetRegistrySubsystem::Get();
	if (RegistryResult.HasValue())
	{
		for (TSharedPtr<UE::ToolsetRegistry::FMCPClientToolset>& Toolset : Toolsets)
		{
			if (Toolset.IsValid())
			{
				RegistryResult.GetValue()->ToolsetRegistry.UnregisterToolset(Toolset);
			}
		}
	}
	Toolsets.Empty();

	Super::Deinitialize();
}
