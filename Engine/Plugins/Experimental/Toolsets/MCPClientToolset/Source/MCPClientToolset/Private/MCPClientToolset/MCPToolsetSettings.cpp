// Copyright Epic Games, Inc. All Rights Reserved.

#include "MCPClientToolset/MCPToolsetSettings.h"

UMCPToolsetSettings::UMCPToolsetSettings()
{
	CategoryName = TEXT("Plugins");
	SectionName  = TEXT("MCP Toolset Servers");
}
