// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module.h"

#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogMCPClientToolset);

void FMCPClientToolsetModule::StartupModule()
{
}

void FMCPClientToolsetModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FMCPClientToolsetModule, MCPClientToolset)
