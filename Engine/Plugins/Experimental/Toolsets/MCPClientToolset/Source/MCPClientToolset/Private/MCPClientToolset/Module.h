// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"

DECLARE_LOG_CATEGORY_EXTERN(LogMCPClientToolset, Log, Log);

class FMCPClientToolsetModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
