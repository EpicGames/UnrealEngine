// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Containers/Map.h"
#include "Containers/Queue.h"
#include "Containers/Ticker.h"
#include "Containers/UnrealString.h"
#include "HttpResultCallback.h"
#include "HttpRouteHandle.h"
#include "HttpServerRequest.h"
#include "Interfaces/IHttpRequest.h"
#include "Misc/DateTime.h"
#include "Templates/SharedPointer.h"
#include "Templates/ValueOrError.h"
#include "ToolsetRegistry/Toolset.h"

class FJsonObject;
class FJsonValue;
class IHttpRouter;

#define UE_API MCPCLIENTTOOLSET_API

namespace UE::ToolsetRegistry
{

/**
 * A toolset that connects to an external MCP (Model Context Protocol) server.
 *
 * Use the async factory Create() to construct an instance. Depending on the FConfig settings
 * the factory will use one of three startup paths:
 *   - OAuth 2.0 Authorization Code + PKCE  (bOAuth = true)
 *   - Streamable HTTP transport             (bStreamableHTTP = true)
 *   - Legacy HTTP+SSE transport             (default)
 *
 * Transport options:
 *   Legacy SSE (MCP pre-2025-03-26):
 *     GET  /sse      — server→client event stream (long-lived)
 *     POST /message  — client→server JSON-RPC requests
 *
 *   Streamable HTTP (MCP 2025-03-26):
 *     POST {ServerUrl}  — single endpoint; response is application/json or text/event-stream
 */
class FMCPClientToolset : public FToolset, public TSharedFromThis<FMCPClientToolset>
{
public:
	/** Configuration for one MCP server connection. */
	struct FConfig
	{
		/** Display name used as the toolset name in the registry. */
		FString Name;
		/** Human-readable description of the toolset. Falls back to Name if empty. */
		FString Description;
		/** Base URL of the server, e.g. "http://localhost:3000". */
		FString ServerUrl;
		/** Optional API key sent as "Authorization: Bearer <ApiKey>". */
		FString ApiKey;
		/** Use Streamable HTTP transport (MCP spec 2025-03-26) instead of legacy SSE. */
		bool    bStreamableHTTP = false;
		/** Use OAuth 2.0 Authorization Code + PKCE instead of a static API key. */
		bool    bOAuth          = false;
		/** OAuth 2.0 client ID (optional — leave empty for dynamic client registration). */
		FString OAuthClientId;
		/** OAuth 2.0 scope string, e.g. "read:me offline_access". */
		FString OAuthScope;
	};

	/**
	 * Opaque tag used so MakeShared can invoke the private-intent constructor.
	 * Call Create() instead of constructing directly.
	 */
	struct FPrivateToken { explicit FPrivateToken() = default; };
	UE_API FMCPClientToolset(FPrivateToken, FConfig InConfig);
	UE_API virtual ~FMCPClientToolset();

	/**
	 * Async factory. Runs the appropriate startup sequence (OAuth → handshake, or direct
	 * handshake), then resolves the future with the ready toolset or an error string.
	 */
	UE_API static TFuture<TValueOrError<TSharedPtr<FMCPClientToolset>, FString>> Create(FConfig Config);

	// FToolset interface
	UE_API virtual TFuture<TValueOrError<FString, FString>> ExecuteToolInternal(
		const FString& ToolName, const FString& JsonInput) override;
	UE_API virtual FString GetJsonSchemaInternal() const override;
	UE_API virtual FString GetToolsetName()    const override;
	UE_API virtual FString GetToolsetVersion() const override;
	UE_API virtual FString GetToolsetDescription() const override;

private:
	// ---- SSE connection ----
	void StartSSEConnection();
	bool OnSSETick(float DeltaTime);
	void OnSSEComplete(FHttpRequestPtr Request, FHttpResponsePtr Response, bool bConnectedSuccessfully);
	void FlushSSEBuffer();
	void DispatchSSEEvent(const FString& EventType, const FString& Data);

	// ---- JSON-RPC helpers ----
	TFuture<TSharedPtr<FJsonObject>> SendRequest(const FString& Method, TSharedPtr<FJsonObject> Params);
	FString BuildSchemaFromMCPTools(const TArray<TSharedPtr<FJsonValue>>& Tools);
	FString MCPResultToString(TSharedPtr<FJsonObject> Result);
	void    RejectAllPendingRequests();

	// ---- Unified auth header (OAuth token > static API key > empty) ----
	FString GetAuthHeader() const;

	// ---- Streamable HTTP transport ----
	void StartStreamableHTTPHandshake();
	TFuture<TSharedPtr<FJsonObject>> SendStreamableHTTPRequest(
		const FString& Method, TSharedPtr<FJsonObject> Params);

	// ---- OAuth 2.0 Authorization Code + PKCE ----
	void StartOAuthFlow();
	void OnOAuthDiscoveryComplete(FHttpRequestPtr Req, FHttpResponsePtr Resp, bool bSuccess);
	void AllocateOAuthPort();
	void RegisterOAuthClient();
	void OnOAuthClientRegistered(FHttpRequestPtr Req, FHttpResponsePtr Resp, bool bSuccess);
	void LaunchOAuthBrowser();
	bool HandleOAuthCallback(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	void ExchangeOAuthCode(const FString& Code);
	void OnOAuthTokenExchangeComplete(FHttpRequestPtr Req, FHttpResponsePtr Resp, bool bSuccess);
	bool OnOAuthTimeout(float DeltaTime);
	void FailOAuth(const FString& Reason);
	void TryRefreshToken();
	void OnRefreshTokenComplete(FHttpRequestPtr Req, FHttpResponsePtr Resp, bool bSuccess);
	void SaveTokensToConfig();
	void LoadTokensFromConfig();

	// ---- Config / capabilities ----
	FConfig Config;
	FString ServerVersion;

	// ---- SSE state ----
	struct FSSEStreamState
	{
		TQueue<TArray<uint8>, EQueueMode::Spsc> Queue;
	};

	TSharedPtr<IHttpRequest, ESPMode::ThreadSafe> SSERequest;
	FTSTicker::FDelegateHandle SSETickerHandle;
	FString MessageEndpointUrl;
	FString SSEBuffer;
	TSharedPtr<FSSEStreamState, ESPMode::ThreadSafe> SSEStreamState;

	// ---- Streamable HTTP state ----
	FString StreamableSessionId;

	// ---- Request tracking ----
	int32 NextRequestId = 0;
	TMap<int32, TSharedPtr<TPromise<TSharedPtr<FJsonObject>>>> PendingRequests;

	// ---- Cached schema ----
	FString CachedSchemaJson;

	// ---- Create() promise, fulfilled after handshake ----
	TSharedPtr<TPromise<TValueOrError<TSharedPtr<FMCPClientToolset>, FString>>> CreatePromise;
	TSharedPtr<FMCPClientToolset> Self;

	// ---- OAuth 2.0 state ----
	enum class EOAuthState : uint8 { Idle, Discovering, WaitingForCode, Exchanging, Complete };

	EOAuthState                OAuthState              = EOAuthState::Idle;
	FString                    AccessToken;
	FString                    RefreshToken;
	FDateTime                  TokenExpiresAt;
	FString                    OAuthCodeVerifier;
	FString                    OAuthCSRFState;
	FString                    OAuthAuthEndpoint;
	FString                    OAuthTokenEndpoint;
	FString                    OAuthRegistrationEndpoint;
	int32                      OAuthLocalPort          = 0;
	TSharedPtr<IHttpRouter>    OAuthRouter;
	FHttpRouteHandle           OAuthRouteHandle;
	FTSTicker::FDelegateHandle OAuthTimeoutHandle;
};

} // namespace UE::ToolsetRegistry

#undef UE_API
