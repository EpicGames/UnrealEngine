// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "Templates/SharedPointer.h"

#include "MCPClientToolsetSubsystem.generated.h"

namespace UE::ToolsetRegistry
{
	class FMCPClientToolset;
}

/** Editor subsystem that reads UMCPToolsetSettings on startup, creates FMCPClientToolset
 *  instances, and registers them with UToolsetRegistrySubsystem. */
UCLASS(MinimalAPI)
class UMCPClientToolsetSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;

private:
	TArray<TSharedPtr<UE::ToolsetRegistry::FMCPClientToolset>> Toolsets;
};
