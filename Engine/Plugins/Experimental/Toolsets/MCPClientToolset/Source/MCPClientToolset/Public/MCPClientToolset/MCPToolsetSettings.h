// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "MCPToolsetSettings.generated.h"

/** Transport protocol for an MCP server connection. */
UENUM(BlueprintType)
enum class EMCPTransport : uint8
{
	SSE            UMETA(DisplayName="Legacy SSE (HTTP+SSE)"),
	StreamableHTTP UMETA(DisplayName="Streamable HTTP")
};

/** Authentication method for an MCP server connection. */
UENUM(BlueprintType)
enum class EMCPAuth : uint8
{
	None        UMETA(DisplayName="None"),
	BearerToken UMETA(DisplayName="Bearer Token (API Key)"),
	OAuth2      UMETA(DisplayName="OAuth 2.0 (Authorization Code + PKCE)")
};

/** Configuration for a single MCP server connection. */
USTRUCT(BlueprintType)
struct FMCPServerConfig
{
	GENERATED_BODY()

	/** Display name for this MCP server (used as the toolset name). */
	UPROPERTY(EditAnywhere, Config, Category = "MCP Server")
	FString Name;

	/** Human-readable description of this toolset, surfaced to the AI as context.
	 *  If empty, the name is used instead. */
	UPROPERTY(EditAnywhere, Config, Category = "MCP Server")
	FString Description;

	/** Base URL of the MCP server (e.g., http://localhost:3000). */
	UPROPERTY(EditAnywhere, Config, Category = "MCP Server")
	FString ServerUrl;

	/** Optional API key sent as "Authorization: Bearer <ApiKey>". */
	UPROPERTY(EditAnywhere, Config, Category = "MCP Server")
	FString ApiKey;

	/** Whether this server config is active. Disabled configs are skipped on startup. */
	UPROPERTY(EditAnywhere, Config, Category = "MCP Server")
	bool bEnabled = true;

	/** Transport protocol to use when connecting to this server. */
	UPROPERTY(EditAnywhere, Config, Category = "Connection")
	EMCPTransport Transport = EMCPTransport::SSE;

	/** Authentication method to use when connecting to this server. */
	UPROPERTY(EditAnywhere, Config, Category = "Authentication")
	EMCPAuth Auth = EMCPAuth::BearerToken;

	/** OAuth 2.0 client ID. Leave empty to use RFC 7591 dynamic client registration
	 *  (the server assigns a client ID automatically — no app registration required). */
	UPROPERTY(EditAnywhere, Config, Category = "Authentication")
	FString OAuthClientId;

	/** OAuth 2.0 scope string, e.g. "read:me offline_access" (required when Auth = OAuth2). */
	UPROPERTY(EditAnywhere, Config, Category = "Authentication")
	FString OAuthScope;
};

/** Editor settings for MCP (Model Context Protocol) server connections.
 *  Configure servers under Editor Preferences > Plugins > MCP Toolset Servers. */
UCLASS(Config = EditorPerProjectUserSettings, meta = (DisplayName = "MCP Toolset Servers"), MinimalAPI)
class UMCPToolsetSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UMCPToolsetSettings();

	/** List of MCP servers to connect to on editor startup. */
	UPROPERTY(EditAnywhere, Config, Category = "MCP Servers")
	TArray<FMCPServerConfig> MCPServers;

	virtual FName GetContainerName() const override { return FName("Editor"); }
	virtual FName GetCategoryName() const override { return FName("Plugins"); }
	virtual FName GetSectionName() const override { return FName("MCP Toolset Servers"); }
};
